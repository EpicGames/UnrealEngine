// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InternationalizationSettingsModel.generated.h"

#define UE_API INTERNATIONALIZATIONSETTINGS_API

UENUM()
enum class ETimezoneSetting : uint8
{
	InternationalDateLineWest UMETA(DisplayName="(UTC-12:00) International Date Line West"),
	CoordinatedUniversalTimeNeg11 UMETA(DisplayName="(UTC-11:00) Coordinated Universal Time -11"),
	Samoa UMETA(DisplayName="(UTC-11:00) Samoa"),
	Hawaii UMETA(DisplayName="(UTC-10:00) Hawaii"),
	Alaska UMETA(DisplayName="(UTC-09:00) Alaska"),
	PacificTime_USCAN UMETA(DisplayName="(UTC-08:00) Pacific Time (US and Canada)"),
	BajaCalifornia UMETA(DisplayName="(UTC-08:00) Baja California"),
	MountainTime_USCAN UMETA(DisplayName="(UTC-07:00) Mountain Time (US and Canada)"),
	Chihuahua_LaPaz_Mazatlan UMETA(DisplayName="(UTC-07:00) Chihuahua, La Paz, Mazatlan"),
	Arizona UMETA(DisplayName="(UTC-07:00) Arizona"),
	Saskatchewan UMETA(DisplayName="(UTC-06:00) Saskatchewan"),
	CentralAmerica UMETA(DisplayName="(UTC-06:00) Central America"),
	CentralTime_USCAN UMETA(DisplayName="(UTC-06:00) Central Time (US and Canada)"),
	Guadalajara_MexicoCity_Monterrey UMETA(DisplayName="(UTC-06:00) Guadalajara, Mexico City, Monterrey"),
	EasternTime_USCAN UMETA(DisplayName="(UTC-05:00) Eastern Time (US and Canada)"),
	Bogota_Lima_Quito UMETA(DisplayName="(UTC-05:00) Bogota, Lima, Quito"),
	Indiana_US UMETA(DisplayName="(UTC-05:00) Indiana (East)"),
	Caracas UMETA(DisplayName="(UTC-04:30) Caracas"),
	AtlanticTime_Canada UMETA(DisplayName="(UTC-04:00) Atlantic Time (Canada)"),
	Cuiaba UMETA(DisplayName="(UTC-04:00) Cuiaba"),
	Santiago UMETA(DisplayName="(UTC-04:00) Santiago"),
	Georgetown_LaPaz_Manaus_SanJuan UMETA(DisplayName="(UTC-04:00) Georgetown, La Paz, Manaus, San Juan"),
	Asuncion UMETA(DisplayName="(UTC-04:00) Asuncion"),
	Newfoundland UMETA(DisplayName="(UTC-03:30) Newfoundland"),
	Brasilia UMETA(DisplayName="(UTC-03:00) Brasilia"),
	Greenland UMETA(DisplayName="(UTC-03:00) Greenland"),
	Montevideo UMETA(DisplayName="(UTC-03:00) Montevideo"),
	Cayenne_Fortaleza UMETA(DisplayName="(UTC-03:00) Cayenne, Fortaleza"),
	BuenosAires UMETA(DisplayName="(UTC-03:00) Buenos Aires"),
	MidAtlantic UMETA(DisplayName="(UTC-02:00) Mid-Atlantic"),
	CoordinatedUniversalTimeNeg02 UMETA(DisplayName="(UTC-02:00) Coordinated Universal Time -02"),
	Azores UMETA(DisplayName="(UTC-01:00) Azores"),
	CaboVerdeIs UMETA(DisplayName="(UTC-01:00) Cabo Verde Is."),
	Dublin_Edinburgh_Lisbon_London UMETA(DisplayName="(UTC) Dublin, Edinburgh, Lisbon, London"),
	Monrovia_Reykjavik UMETA(DisplayName="(UTC) Monrovia, Reykjavik"),
	Casablanca UMETA(DisplayName="(UTC) Casablanca"),
	UTC UMETA(DisplayName="(UTC) Coordinated Universal Time"),
	Belgrade_Bratislava_Budapest_Ljubljana_Prague UMETA(DisplayName="(UTC+01:00) Belgrade, Bratislava, Budapest, Ljubljana, Prague"),
	Sarajevo_Skopje_Warsaw_Zagreb UMETA(DisplayName="(UTC+01:00) Sarajevo, Skopje, Warsaw, Zagreb"),
	Brussels_Copenhagen_Madrid_Paris UMETA(DisplayName="(UTC+01:00) Brussels, Copenhagen, Madrid, Paris"),
	WestCentralAfrica UMETA(DisplayName="(UTC+01:00) West Central Africa"),
	Amsterdam_Berlin_Bern_Rome_Stockholm_Vienna UMETA(DisplayName="(UTC+01:00) Amsterdam, Berlin, Bern, Rome, Stockholm, Vienna"),
	Windhoek UMETA(DisplayName="(UTC+01:00) Windhoek"),
	Minsk UMETA(DisplayName="(UTC+02:00) Minsk"),
	Cairo UMETA(DisplayName="(UTC+02:00) Cairo"),
	Helsinki_Kyiv_Riga_Sofia_Tallinn_Vilnius UMETA(DisplayName="(UTC+02:00) Helsinki, Kyiv, Riga, Sofia, Tallinn, Vilnius"),
	Athens_Bucharest UMETA(DisplayName="(UTC+02:00) Athens, Bucharest"),
	Jerusalem UMETA(DisplayName="(UTC+02:00) Jerusalem"),
	Amman UMETA(DisplayName="(UTC+02:00) Amman"),
	Beirut UMETA(DisplayName="(UTC+02:00) Beirut"),
	Harare_Pretoria UMETA(DisplayName="(UTC+02:00) Harare, Pretoria"),
	Damascus UMETA(DisplayName="(UTC+02:00) Damascus"),
	Istanbul UMETA(DisplayName="(UTC+02:00) Istanbul"),
	Kuwait_Riyadh UMETA(DisplayName="(UTC+03:00) Kuwait, Riyadh"),
	Baghdad UMETA(DisplayName="(UTC+03:00) Baghdad"),
	Nairobi UMETA(DisplayName="(UTC+03:00) Nairobi"),
	Kaliningrad UMETA(DisplayName="(UTC+02:00) Kaliningrad"),
	Tehran UMETA(DisplayName="(UTC+03:30) Tehran"),
	Moscow_StPetersburg_Volgograd UMETA(DisplayName="(UTC+03:00) Moscow, St. Petersburg, Volgograd"),
	AbuDhabi_Muscat UMETA(DisplayName="(UTC+04:00) Abu Dhabi, Muscat"),
	Baku UMETA(DisplayName="(UTC+04:00) Baku"),
	Yerevan UMETA(DisplayName="(UTC+04:00) Yerevan"),
	Tbilisi UMETA(DisplayName="(UTC+04:00) Tbilisi"),
	PortLouis UMETA(DisplayName="(UTC+04:00) Port Louis"),
	Kabul UMETA(DisplayName="(UTC+04:30) Kabul"),
	Tashkent UMETA(DisplayName="(UTC+05:00) Tashkent"),
	Islamabad_Karachi UMETA(DisplayName="(UTC+05:00) Islamabad, Karachi"),
	Chennai_Kolkata_Mumbai_NewDelhi UMETA(DisplayName="(UTC+05:30) Chennai, Kolkata, Mumbai, New Delhi"),
	SriJayawardenepura UMETA(DisplayName="(UTC+05:30) Sri Jayawardenepura"),
	Kathmandu UMETA(DisplayName="(UTC+05:45) Kathmandu"),
	Ekaterinburg UMETA(DisplayName="(UTC+05:00) Ekaterinburg"),
	Astana UMETA(DisplayName="(UTC+06:00) Astana"),
	Dhaka UMETA(DisplayName="(UTC+06:00) Dhaka"),
	Yangon_Rangoon UMETA(DisplayName="(UTC+06:30) Yangon (Rangoon)"),
	Bangkok_Hanoi_Jakarta UMETA(DisplayName="(UTC+07:00) Bangkok, Hanoi, Jakarta"),
	Krasnoyarsk UMETA(DisplayName="(UTC+07:00) Krasnoyarsk"),
	Novosibirsk UMETA(DisplayName="(UTC+07:00) Novosibirsk"),
	Beijing_Chongqing_HongKong_Urumqi UMETA(DisplayName="(UTC+08:00) Beijing, Chongqing, Hong Kong, Urumqi"),
	KualaLumpur_Singapore UMETA(DisplayName="(UTC+08:00) Kuala Lumpur, Singapore"),
	Taipei UMETA(DisplayName="(UTC+08:00) Taipei"),
	Perth UMETA(DisplayName="(UTC+08:00) Perth"),
	Ulaanbaatar UMETA(DisplayName="(UTC+08:00) Ulaanbaatar"),
	Irkutsk UMETA(DisplayName="(UTC+08:00) Irkutsk"),
	Seoul UMETA(DisplayName="(UTC+09:00) Seoul"),
	Osaka_Sapporo_Tokyo UMETA(DisplayName="(UTC+09:00) Osaka, Sapporo, Tokyo"),
	Darwin UMETA(DisplayName="(UTC+09:30) Darwin"),
	Adelaide UMETA(DisplayName="(UTC+09:30) Adelaide"),
	Yakutsk UMETA(DisplayName="(UTC+9:00) Yakutsk"),
	Canberra_Melbourne_Sydney UMETA(DisplayName="(UTC+10:00) Canberra, Melbourne, Sydney"),
	Brisbane UMETA(DisplayName="(UTC+10:00) Brisbane"),
	Hobart UMETA(DisplayName="(UTC+10:00) Hobart"),
	Guam_PortMoresby UMETA(DisplayName="(UTC+10:00) Guam, Port Moresby"),
	Vladivostok UMETA(DisplayName="(UTC+10:00) Vladivostok"),
	SolomonIs_NewCaledonia UMETA(DisplayName="(UTC+11:00) Solomon Is., New Caledonia"),
	Magadan UMETA(DisplayName="(UTC+11:00) Magadan"),
	Fiji UMETA(DisplayName="(UTC+12:00) Fiji"),
	Auckland_Wellington UMETA(DisplayName="(UTC+12:00) Auckland, Wellington"),
	CoordinatedUniversalTime12 UMETA(DisplayName="(UTC+12:00) Coordinated Universal Time +12"),
	Nukualofa UMETA(DisplayName="(UTC+13:00) Nuku'alofa"),

	LocalTime
};

/**
 * Implements loading and saving of internationalization settings.
 */
UCLASS(MinimalAPI, config=EditorSettings)
class UInternationalizationSettingsModel
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UE_API void ResetToDefault();
	UE_API bool GetEditorLanguage(FString& OutEditorLanguage) const;
	UE_API void SetEditorLanguage(const FString& InEditorLanguage);
	UE_API bool GetEditorLocale(FString& OutEditorLocale) const;
	UE_API void SetEditorLocale(const FString& InEditorLocale);
	UE_API bool GetPreviewGameLanguage(FString& OutPreviewGameLanguage) const;
	UE_API void SetPreviewGameLanguage(const FString& InPreviewGameLanguage);
	UE_API bool ShouldUseLocalizedNumericInput() const;
	UE_API void SetShouldUseLocalizedNumericInput(const bool Value);
	UE_API bool ShouldUseLocalizedPropertyNames() const;
	UE_API void SetShouldUseLocalizedPropertyNames(const bool Value);
	UE_API bool ShouldUseLocalizedNodeAndPinNames() const;
	UE_API void SetShouldUseLocalizedNodeAndPinNames(const bool Value);

	UE_API int32 GetTimezoneValue() const;

public:
	/**
	 * Timezone to use for display purposes in FDateTime
	 */
	UPROPERTY(EditAnywhere, Config, Category = Time)
	ETimezoneSetting DisplayTimezone;

};

#undef UE_API
