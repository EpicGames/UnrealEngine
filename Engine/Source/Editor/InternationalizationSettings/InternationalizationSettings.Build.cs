// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class InternationalizationSettings : ModuleRules
	{
		public InternationalizationSettings(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"InputCore",
					"Engine",
					"Slate",
					"SlateCore",
					"AppFramework",
					"PropertyEditor",
					"Localization",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SettingsEditor"
				}
			);
		}
	}
}
