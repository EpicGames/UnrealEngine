// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternationalizationSettingsModel.h"

#include "CoreGlobals.h"
#include "Internationalization/TextLocalizationManager.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternationalizationSettingsModel)

/* UInternationalizationSettingsModel interface
 *****************************************************************************/

UInternationalizationSettingsModel::UInternationalizationSettingsModel( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	DisplayTimezone = ETimezoneSetting::LocalTime;
}

void UInternationalizationSettingsModel::ResetToDefault()
{
	// Inherit editor culture from engine settings. Empty otherwise.
	FString SavedLanguageName;
	GConfig->GetString(TEXT("Internationalization"), TEXT("Language"), SavedLanguageName, GEngineIni);
	GConfig->SetString(TEXT("Internationalization"), TEXT("Language"), *SavedLanguageName, GEditorSettingsIni);

	FString SavedLocaleName;
	GConfig->GetString(TEXT("Internationalization"), TEXT("Locale"), SavedLocaleName, GEngineIni);
	GConfig->SetString(TEXT("Internationalization"), TEXT("Locale"), *SavedLocaleName, GEditorSettingsIni);

	FString SavedCultureName;
	GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), SavedCultureName, GEngineIni);
	GConfig->SetString( TEXT("Internationalization"), TEXT("Culture"), *SavedCultureName, GEditorSettingsIni );

	GConfig->SetBool( TEXT("Internationalization"), TEXT("ShouldUseLocalizedNumericInput"), true, GEditorSettingsIni );
	GConfig->SetBool( TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), true, GEditorSettingsIni );
	GConfig->SetBool( TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), true, GEditorSettingsIni );

	GConfig->Flush(false, GEditorSettingsIni);

	FTextLocalizationManager::Get().ConfigureGameLocalizationPreviewLanguage(FString());
}

bool UInternationalizationSettingsModel::GetEditorLanguage(FString& OutEditorLanguage) const
{
	return GConfig->GetString(TEXT("Internationalization"), TEXT("Language"), OutEditorLanguage, GEditorSettingsIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), OutEditorLanguage, GEditorSettingsIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Language"), OutEditorLanguage, GEngineIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), OutEditorLanguage, GEngineIni);
}

void UInternationalizationSettingsModel::SetEditorLanguage(const FString& InEditorLanguage)
{
	GConfig->SetString( TEXT("Internationalization"), TEXT("Language"), *InEditorLanguage, GEditorSettingsIni );
	GConfig->SetString( TEXT("Internationalization"), TEXT("Culture"), TEXT(""), GEditorSettingsIni ); // Clear legacy setting
	GConfig->Flush(false, GEditorSettingsIni);
}

bool UInternationalizationSettingsModel::GetEditorLocale(FString& OutEditorLocale) const
{
	return GConfig->GetString(TEXT("Internationalization"), TEXT("Locale"), OutEditorLocale, GEditorSettingsIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), OutEditorLocale, GEditorSettingsIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Locale"), OutEditorLocale, GEngineIni)
		|| GConfig->GetString(TEXT("Internationalization"), TEXT("Culture"), OutEditorLocale, GEngineIni);
}

void UInternationalizationSettingsModel::SetEditorLocale(const FString& InEditorLocale)
{
	GConfig->SetString( TEXT("Internationalization"), TEXT("Locale"), *InEditorLocale, GEditorSettingsIni );
	GConfig->SetString( TEXT("Internationalization"), TEXT("Culture"), TEXT(""), GEditorSettingsIni ); // Clear legacy setting
	GConfig->Flush(false, GEditorSettingsIni);
}

bool UInternationalizationSettingsModel::GetPreviewGameLanguage(FString& OutPreviewGameLanguage) const
{
	OutPreviewGameLanguage = FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage();
	return true;
}

void UInternationalizationSettingsModel::SetPreviewGameLanguage(const FString& InPreviewGameLanguage)
{
	FTextLocalizationManager::Get().ConfigureGameLocalizationPreviewLanguage(InPreviewGameLanguage);
}

bool UInternationalizationSettingsModel::ShouldUseLocalizedNumericInput() const
{
	bool bShouldUseLocalizedNumericInput = false;
	GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNumericInput"), bShouldUseLocalizedNumericInput, GEditorSettingsIni);
	return bShouldUseLocalizedNumericInput;
}

void UInternationalizationSettingsModel::SetShouldUseLocalizedNumericInput(const bool Value)
{
	GConfig->SetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNumericInput"), Value, GEditorSettingsIni);
	GConfig->Flush(false, GEditorSettingsIni);
}

bool UInternationalizationSettingsModel::ShouldUseLocalizedPropertyNames() const
{
	bool bShouldUseLocalizedPropertyNames = false;
	GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), bShouldUseLocalizedPropertyNames, GEditorSettingsIni);
	return bShouldUseLocalizedPropertyNames;
}

void UInternationalizationSettingsModel::SetShouldUseLocalizedPropertyNames(const bool Value)
{
	GConfig->SetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), Value, GEditorSettingsIni);
	GConfig->Flush(false, GEditorSettingsIni);
}

bool UInternationalizationSettingsModel::ShouldUseLocalizedNodeAndPinNames() const
{
	bool bShouldUseLocalizedNodeAndPinNames = false;
	GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), bShouldUseLocalizedNodeAndPinNames, GEditorSettingsIni);
	return bShouldUseLocalizedNodeAndPinNames;
}

void UInternationalizationSettingsModel::SetShouldUseLocalizedNodeAndPinNames(const bool Value)
{
	GConfig->SetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedNodeAndPinNames"), Value, GEditorSettingsIni);
	GConfig->Flush(false, GEditorSettingsIni);
}

int32 UInternationalizationSettingsModel::GetTimezoneValue() const
{
	switch (DisplayTimezone)
	{
	case ETimezoneSetting::InternationalDateLineWest:
		return -1200;
	case ETimezoneSetting::CoordinatedUniversalTimeNeg11:
	case ETimezoneSetting::Samoa:
		return -1100;
	case ETimezoneSetting::Hawaii:
		return -1000;
	case ETimezoneSetting::Alaska:
		return -900;
	case ETimezoneSetting::PacificTime_USCAN:
	case ETimezoneSetting::BajaCalifornia:
		return -800;
	case ETimezoneSetting::MountainTime_USCAN:
	case ETimezoneSetting::Chihuahua_LaPaz_Mazatlan:
	case ETimezoneSetting::Arizona:
		return -700;
	case ETimezoneSetting::Saskatchewan:
	case ETimezoneSetting::CentralAmerica:
	case ETimezoneSetting::CentralTime_USCAN:
	case ETimezoneSetting::Guadalajara_MexicoCity_Monterrey:
		return -600;
	case ETimezoneSetting::EasternTime_USCAN:
	case ETimezoneSetting::Bogota_Lima_Quito:
	case ETimezoneSetting::Indiana_US:
		return -500;
	case ETimezoneSetting::Caracas:
		return -430;
	case ETimezoneSetting::AtlanticTime_Canada:
	case ETimezoneSetting::Cuiaba:
	case ETimezoneSetting::Santiago:
	case ETimezoneSetting::Georgetown_LaPaz_Manaus_SanJuan:
	case ETimezoneSetting::Asuncion:
		return -400;
	case ETimezoneSetting::Newfoundland:
		return -330;
	case ETimezoneSetting::Brasilia:
	case ETimezoneSetting::Greenland:
	case ETimezoneSetting::Montevideo:
	case ETimezoneSetting::Cayenne_Fortaleza:
	case ETimezoneSetting::BuenosAires:
		return -300;
	case ETimezoneSetting::MidAtlantic:
	case ETimezoneSetting::CoordinatedUniversalTimeNeg02:
		return -200;
	case ETimezoneSetting::Azores:
	case ETimezoneSetting::CaboVerdeIs:
		return -100;
	case ETimezoneSetting::Dublin_Edinburgh_Lisbon_London:
	case ETimezoneSetting::Monrovia_Reykjavik:
	case ETimezoneSetting::Casablanca:
	case ETimezoneSetting::UTC:
		return 0;
	case ETimezoneSetting::Belgrade_Bratislava_Budapest_Ljubljana_Prague:
	case ETimezoneSetting::Sarajevo_Skopje_Warsaw_Zagreb:
	case ETimezoneSetting::Brussels_Copenhagen_Madrid_Paris:
	case ETimezoneSetting::WestCentralAfrica:
	case ETimezoneSetting::Amsterdam_Berlin_Bern_Rome_Stockholm_Vienna:
	case ETimezoneSetting::Windhoek:
		return 100;
	case ETimezoneSetting::Minsk:
	case ETimezoneSetting::Cairo:
	case ETimezoneSetting::Helsinki_Kyiv_Riga_Sofia_Tallinn_Vilnius:
	case ETimezoneSetting::Athens_Bucharest:
	case ETimezoneSetting::Jerusalem:
	case ETimezoneSetting::Amman:
	case ETimezoneSetting::Beirut:
	case ETimezoneSetting::Harare_Pretoria:
	case ETimezoneSetting::Damascus:
	case ETimezoneSetting::Istanbul:
	case ETimezoneSetting::Kaliningrad:
		return 200;
	case ETimezoneSetting::Moscow_StPetersburg_Volgograd:
	case ETimezoneSetting::Kuwait_Riyadh:
	case ETimezoneSetting::Baghdad:
	case ETimezoneSetting::Nairobi:
		return 300;
	case ETimezoneSetting::Tehran:
		return 330;
	case ETimezoneSetting::AbuDhabi_Muscat:
	case ETimezoneSetting::Baku:
	case ETimezoneSetting::Yerevan:
	case ETimezoneSetting::Tbilisi:
	case ETimezoneSetting::PortLouis:
		return 400;
	case ETimezoneSetting::Kabul:
		return 430;
	case ETimezoneSetting::Ekaterinburg:
	case ETimezoneSetting::Tashkent:
	case ETimezoneSetting::Islamabad_Karachi:
		return 500;
	case ETimezoneSetting::Chennai_Kolkata_Mumbai_NewDelhi:
	case ETimezoneSetting::SriJayawardenepura:
		return 530;
	case ETimezoneSetting::Kathmandu:
		return 545;
	case ETimezoneSetting::Astana:
	case ETimezoneSetting::Dhaka:
		return 600;
	case ETimezoneSetting::Yangon_Rangoon:
		return 630;
	case ETimezoneSetting::Krasnoyarsk:
	case ETimezoneSetting::Novosibirsk:
	case ETimezoneSetting::Bangkok_Hanoi_Jakarta:
		return 700;
	case ETimezoneSetting::Irkutsk:
	case ETimezoneSetting::Beijing_Chongqing_HongKong_Urumqi:
	case ETimezoneSetting::KualaLumpur_Singapore:
	case ETimezoneSetting::Taipei:
	case ETimezoneSetting::Perth:
	case ETimezoneSetting::Ulaanbaatar:
		return 800;
	case ETimezoneSetting::Yakutsk:
	case ETimezoneSetting::Seoul:
	case ETimezoneSetting::Osaka_Sapporo_Tokyo:
		return 900;
	case ETimezoneSetting::Darwin:
	case ETimezoneSetting::Adelaide:
		return 930;
	case ETimezoneSetting::Vladivostok:
	case ETimezoneSetting::Canberra_Melbourne_Sydney:
	case ETimezoneSetting::Brisbane:
	case ETimezoneSetting::Hobart:
	case ETimezoneSetting::Guam_PortMoresby:
		return 1000;
	case ETimezoneSetting::Magadan:
	case ETimezoneSetting::SolomonIs_NewCaledonia:
		return 1100;
	case ETimezoneSetting::Fiji:
	case ETimezoneSetting::Auckland_Wellington:
	case ETimezoneSetting::CoordinatedUniversalTime12:
		return 1200;
	case ETimezoneSetting::Nukualofa:
		return 1300;
	case ETimezoneSetting::LocalTime:
	{
		// This is definitely a hack, but our FPlatformTime doesn't do timezones
		const FDateTime LocalNow = FDateTime::Now();
		const FDateTime UTCNow = FDateTime::UtcNow();

		const FTimespan Difference = LocalNow - UTCNow;

		const int32 MinutesDifference = FMath::RoundToInt(Difference.GetTotalMinutes());

		const int32 Hours = MinutesDifference / 60;
		const int32 Minutes = MinutesDifference % 60;

		const int32 Timezone = (Hours * 100) + Minutes;
		return Timezone;
	}

	default:
		return 0;
	}
}
