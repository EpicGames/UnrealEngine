// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternationalizationSettingsModelDetails.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraphSchema.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Culture.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextLocalizationManager.h"
#include "InternationalizationSettingsModel.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SCulturePicker.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "InternationalizationSettingsModelDetails"

TSharedRef<IDetailCustomization> FInternationalizationSettingsModelDetails::MakeInstance()
{
	return MakeShareable(new FInternationalizationSettingsModelDetails);
}

namespace
{
	struct FLocalizedCulturesFlyweight
	{
		TArray<FCultureRef> LocalizedCulturesForEditor;
		TArray<FCultureRef> LocalizedCulturesForGame;

		FLocalizedCulturesFlyweight()
		{
			constexpr bool bIncludeDerivedCultures = false;

			{
				const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Editor);
				LocalizedCulturesForEditor = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, bIncludeDerivedCultures);
			}
			{
				const TArray<FString> LocalizedCultureNames = FTextLocalizationManager::Get().GetLocalizedCultureNames(ELocalizationLoadFlags::Game);
				LocalizedCulturesForGame = FInternationalization::Get().GetAvailableCultures(LocalizedCultureNames, bIncludeDerivedCultures);
			}
		}
	};

	class SEditorLanguageComboButton : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SEditorLanguageComboButton){}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, const TWeakObjectPtr<UInternationalizationSettingsModel>& InSettingsModel, const TSharedRef<FLocalizedCulturesFlyweight>& InLocalizedCulturesFlyweight)
		{
			SettingsModel = InSettingsModel;
			LocalizedCulturesFlyweight = InLocalizedCulturesFlyweight;

			ChildSlot
			[
				SNew(SCulturePickerCombo)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.SelectedCulture(this, &SEditorLanguageComboButton::GetSelectedCulture)
				.OnSelectionChanged(this, &SEditorLanguageComboButton::OnSelectionChanged)
				.IsCulturePickable(this, &SEditorLanguageComboButton::IsCulturePickable)
				.DisplayNameFormat(SCulturePicker::ECultureDisplayFormat::ActiveAndNativeCultureDisplayName)
				.ViewMode(SCulturePicker::ECulturesViewMode::Flat)
			];
		}

	private:
		TWeakObjectPtr<UInternationalizationSettingsModel> SettingsModel;
		TSharedPtr<FLocalizedCulturesFlyweight> LocalizedCulturesFlyweight;

	private:
		FCulturePtr GetSelectedCulture() const
		{
			return FInternationalization::Get().GetCurrentLanguage();
		}

		void OnSelectionChanged(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo)
		{
			if (SettingsModel.IsValid())
			{
				SettingsModel->SetEditorLanguage(SelectedCulture.IsValid() ? SelectedCulture->GetName() : TEXT(""));
				if (SelectedCulture.IsValid())
				{
					FInternationalization& I18N = FInternationalization::Get();
					I18N.SetCurrentLanguage(SelectedCulture->GetName());

					// Find all Schemas and force a visualization cache clear
					for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
					{
						UClass* CurrentClass = *ClassIt;

						if (UEdGraphSchema* Schema = Cast<UEdGraphSchema>(CurrentClass->GetDefaultObject()))
						{
							Schema->ForceVisualizationCacheClear();
						}
					}
				}
			}
		}

		bool IsCulturePickable(FCulturePtr Culture)
		{
			TArray<FString> CultureNames = Culture->GetPrioritizedParentCultureNames();
			for (const FString& CultureName : CultureNames)
			{
				if (LocalizedCulturesFlyweight->LocalizedCulturesForEditor.Contains(Culture))
				{
					return true;
				}
			}
			return false;
		}
	};

	class SEditorLocaleComboButton : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SEditorLocaleComboButton){}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, const TWeakObjectPtr<UInternationalizationSettingsModel>& InSettingsModel, const TSharedRef<FLocalizedCulturesFlyweight>& InLocalizedCulturesFlyweight)
		{
			SettingsModel = InSettingsModel;
			LocalizedCulturesFlyweight = InLocalizedCulturesFlyweight;

			ChildSlot
			[
				SNew(SCulturePickerCombo)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.SelectedCulture(this, &SEditorLocaleComboButton::GetSelectedCulture)
				.OnSelectionChanged(this, &SEditorLocaleComboButton::OnSelectionChanged)
				.DisplayNameFormat(SCulturePicker::ECultureDisplayFormat::ActiveAndNativeCultureDisplayName)
			];
		}

	private:
		TWeakObjectPtr<UInternationalizationSettingsModel> SettingsModel;
		TSharedPtr<FLocalizedCulturesFlyweight> LocalizedCulturesFlyweight;

	private:
		FCulturePtr GetSelectedCulture() const
		{
			return FInternationalization::Get().GetCurrentLocale();
		}

		void OnSelectionChanged(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo)
		{
			if (SettingsModel.IsValid())
			{
				SettingsModel->SetEditorLocale(SelectedCulture.IsValid() ? SelectedCulture->GetName() : TEXT(""));
				if (SelectedCulture.IsValid())
				{
					FInternationalization& I18N = FInternationalization::Get();
					I18N.SetCurrentLocale(SelectedCulture->GetName());

					// Find all Schemas and force a visualization cache clear
					for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
					{
						UClass* CurrentClass = *ClassIt;

						if (UEdGraphSchema* Schema = Cast<UEdGraphSchema>(CurrentClass->GetDefaultObject()))
						{
							Schema->ForceVisualizationCacheClear();
						}
					}
				}
			}
		}
	};

	class SPreviewGameLanguageComboButton : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SPreviewGameLanguageComboButton){}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, const TWeakObjectPtr<UInternationalizationSettingsModel>& InSettingsModel, const TSharedRef<FLocalizedCulturesFlyweight>& InLocalizedCulturesFlyweight)
		{
			SettingsModel = InSettingsModel;
			LocalizedCulturesFlyweight = InLocalizedCulturesFlyweight;

			ChildSlot
			[
				SNew(SCulturePickerCombo)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.SelectedCulture(this, &SPreviewGameLanguageComboButton::GetSelectedCulture)
				.OnSelectionChanged(this, &SPreviewGameLanguageComboButton::OnSelectionChanged)
				.IsCulturePickable(this, &SPreviewGameLanguageComboButton::IsCulturePickable)
				.DisplayNameFormat(SCulturePicker::ECultureDisplayFormat::ActiveAndNativeCultureDisplayName)
				.ViewMode(SCulturePicker::ECulturesViewMode::Flat)
				.CanSelectNone(true)
			];
		}

	private:
		TWeakObjectPtr<UInternationalizationSettingsModel> SettingsModel;
		TSharedPtr<FLocalizedCulturesFlyweight> LocalizedCulturesFlyweight;

	private:
		FCulturePtr GetSelectedCulture() const
		{
			FString PreviewGameLanguage;
			if (SettingsModel.IsValid())
			{
				SettingsModel->GetPreviewGameLanguage(PreviewGameLanguage);
			}

			FCulturePtr Culture;
			if (!PreviewGameLanguage.IsEmpty())
			{
				Culture = FInternationalization::Get().GetCulture(PreviewGameLanguage);
			}

			return Culture;
		}

		void OnSelectionChanged(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo)
		{
			if (SettingsModel.IsValid())
			{
				SettingsModel->SetPreviewGameLanguage(SelectedCulture.IsValid() ? SelectedCulture->GetName() : TEXT(""));

				if (FTextLocalizationManager::Get().ShouldGameLocalizationPreviewAutoEnable() || FTextLocalizationManager::Get().IsGameLocalizationPreviewEnabled())
				{
					// Enable the preview again for the newly set culture
					FTextLocalizationManager::Get().EnableGameLocalizationPreview();
				}
			}
		}

		bool IsCulturePickable(FCulturePtr Culture)
		{
			TArray<FString> CultureNames = Culture->GetPrioritizedParentCultureNames();
			for (const FString& CultureName : CultureNames)
			{
				if (LocalizedCulturesFlyweight->LocalizedCulturesForGame.Contains(Culture))
				{
					return true;
				}
			}
			return false;
		}
	};
}

void FInternationalizationSettingsModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TWeakObjectPtr<UInternationalizationSettingsModel> SettingsModel = [&]()
	{
		TArray< TWeakObjectPtr<UObject> > ObjectsBeingCustomized;
		DetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		check(ObjectsBeingCustomized.Num() == 1);
		return Cast<UInternationalizationSettingsModel>(ObjectsBeingCustomized.Top().Get());
	}();

	IDetailCategoryBuilder& DetailCategoryBuilder = DetailLayout.EditCategory("Internationalization", LOCTEXT("InternationalizationCategory", "Internationalization"));

	const TSharedRef<FLocalizedCulturesFlyweight> LocalizedCulturesFlyweight = MakeShareable(new FLocalizedCulturesFlyweight());

	// Editor Language Setting
	const FText EditorLanguageSettingDisplayName = LOCTEXT("EditorLanguageSettingDisplayName", "Editor Language");
	const FText EditorLanguageSettingToolTip = LOCTEXT("EditorLanguageSettingToolTip", "The language that the Editor should use for localization (the display language).");
	DetailCategoryBuilder.AddCustomRow(EditorLanguageSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(EditorLanguageSettingDisplayName)
			.ToolTipText(EditorLanguageSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditorLanguageComboButton, SettingsModel, LocalizedCulturesFlyweight)
		];

	// Editor Locale Setting
	const FText EditorLocaleSettingDisplayName = LOCTEXT("EditorLocaleSettingDisplayName", "Editor Locale");
	const FText EditorLocaleSettingToolTip = LOCTEXT("EditorLocaleSettingToolTip", "The locale that the Editor should use for internationalization (numbers, dates, times, etc).");
	DetailCategoryBuilder.AddCustomRow(EditorLocaleSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(EditorLocaleSettingDisplayName)
			.ToolTipText(EditorLocaleSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditorLocaleComboButton, SettingsModel, LocalizedCulturesFlyweight)
		];

	// Preview Game Language Setting
	const FText PreviewGameLanguageSettingDisplayName = LOCTEXT("PreviewGameLanguageSettingDisplayName", "Preview Game Language");
	const FText PreviewGameLanguageSettingToolTip = LOCTEXT("PreviewGameLanguageSettingToolTip", "The language to preview game localization in");
	DetailCategoryBuilder.AddCustomRow(PreviewGameLanguageSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(PreviewGameLanguageSettingDisplayName)
			.ToolTipText(PreviewGameLanguageSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SPreviewGameLanguageComboButton, SettingsModel, LocalizedCulturesFlyweight)
		];

	// Localized Numeric Input
	const FText NumericInputSettingDisplayName = LOCTEXT("LocalizedNumericInputLabel", "Use Localized Numeric Input");
	const FText NumericInputSettingToolTip = LOCTEXT("LocalizedNumericInputTooltip", "Allow numbers to be displayed and modified in the format for the current locale, rather than in the language agnostic format.");
	DetailCategoryBuilder.AddCustomRow(NumericInputSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(NumericInputSettingDisplayName)
			.ToolTipText(NumericInputSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(300.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([=]()
			{
				return SettingsModel.IsValid() && SettingsModel->ShouldUseLocalizedNumericInput() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.ToolTipText(NumericInputSettingToolTip)
			.OnCheckStateChanged_Lambda([=](ECheckBoxState State)
			{
				if (SettingsModel.IsValid())
				{
					SettingsModel->SetShouldUseLocalizedNumericInput(State == ECheckBoxState::Checked);
				}
			})
		];

	// Localized Property Names
	const FText PropertyNamesSettingDisplayName = LOCTEXT("LocalizedEditorPropertyNamesLabel", "Use Localized Property Names");
	const FText PropertyNamesSettingToolTip = LOCTEXT("LocalizedEditorPropertyNamesTooltip", "Toggle showing localized property names.");
	DetailCategoryBuilder.AddCustomRow(PropertyNamesSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(PropertyNamesSettingDisplayName)
			.ToolTipText(PropertyNamesSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(300.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([=]()
			{
				return SettingsModel.IsValid() && SettingsModel->ShouldUseLocalizedPropertyNames() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.ToolTipText(PropertyNamesSettingToolTip)
			.OnCheckStateChanged_Lambda([=](ECheckBoxState State)
			{
				if (SettingsModel.IsValid())
				{
					SettingsModel->SetShouldUseLocalizedPropertyNames(State == ECheckBoxState::Checked);
					FTextLocalizationManager::Get().RefreshResources();
				}
			})
		];

	// Localized Node and Pin Names
	const FText NodeAndPinNamesSettingDisplayName = LOCTEXT("LocalizedGraphEditorNodeAndPinNamesLabel", "Use Localized Graph Editor Node and Pin Names");
	const FText NodeAndPinNamesSettingToolTip = LOCTEXT("LocalizedGraphEditorNodeAndPinNamesTooltip", "Toggle localized node and pin names in all graph editors.");
	DetailCategoryBuilder.AddCustomRow(NodeAndPinNamesSettingDisplayName)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(NodeAndPinNamesSettingDisplayName)
			.ToolTipText(NodeAndPinNamesSettingToolTip)
			.Font(DetailLayout.GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(300.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([=]()
			{
				return SettingsModel.IsValid() && SettingsModel->ShouldUseLocalizedNodeAndPinNames() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.ToolTipText(NodeAndPinNamesSettingToolTip)
			.OnCheckStateChanged_Lambda([=](ECheckBoxState State)
			{
				if (SettingsModel.IsValid())
				{
					SettingsModel->SetShouldUseLocalizedNodeAndPinNames(State == ECheckBoxState::Checked);

					// Find all Schemas and force a visualization cache clear
					for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
					{
						UClass* CurrentClass = *ClassIt;

						if (UEdGraphSchema* Schema = Cast<UEdGraphSchema>(CurrentClass->GetDefaultObject()))
						{
							Schema->ForceVisualizationCacheClear();
						}
					}
				}
			})
		];
}

#undef LOCTEXT_NAMESPACE
