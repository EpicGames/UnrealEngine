// Copyright Epic Games, Inc. All Rights Reserved.

#include "InternationalizationSettingsModule.h"

#include "Delegates/Delegate.h"
#include "InternationalizationSettingsModelDetails.h"
#include "UserGeneratedContentLocalizationDescriptorDetails.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"


class FInternationalizationSettingsModule : public IInternationalizationSettingsModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FInternationalizationSettingsModule, InternationalizationSettings)

void FInternationalizationSettingsModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("InternationalizationSettingsModel", FOnGetDetailCustomizationInstance::CreateStatic(&FInternationalizationSettingsModelDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("UserGeneratedContentLocalizationDescriptor", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FUserGeneratedContentLocalizationDescriptorDetails::MakeInstance));
}

void FInternationalizationSettingsModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("InternationalizationSettingsModel");
		PropertyModule.UnregisterCustomPropertyTypeLayout("UserGeneratedContentLocalizationDescriptor");
	}
}