// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCulturePicker.h"

#include "Containers/Map.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;

#define LOCTEXT_NAMESPACE "CulturePicker"

namespace UE::Internationalization::Private
{

FString GetCultureDisplayName(const FCultureRef& Culture, SCulturePicker::ECultureDisplayFormat DisplayNameFormat, const bool bIsRootItem)
{
	const FString DisplayName = Culture->GetDisplayName();
	if (DisplayNameFormat == SCulturePicker::ECultureDisplayFormat::ActiveCultureDisplayName)
	{
		return DisplayName;
	}

	const FString NativeName = Culture->GetNativeName();
	if (DisplayNameFormat == SCulturePicker::ECultureDisplayFormat::NativeCultureDisplayName)
	{
		return NativeName;
	}

	if (DisplayNameFormat == SCulturePicker::ECultureDisplayFormat::ActiveAndNativeCultureDisplayName)
	{
		// Only show both names if they're different (to avoid repetition), and we're a root item (to avoid noise)
		return (bIsRootItem && !NativeName.Equals(DisplayName, ESearchCase::CaseSensitive))
			? FString::Printf(TEXT("%s [%s]"), *DisplayName, *NativeName)
			: DisplayName;
	}

	if (DisplayNameFormat == SCulturePicker::ECultureDisplayFormat::NativeAndActiveCultureDisplayName)
	{
		// Only show both names if they're different (to avoid repetition), and we're a root item (to avoid noise)
		return (bIsRootItem && !NativeName.Equals(DisplayName, ESearchCase::CaseSensitive))
			? FString::Printf(TEXT("%s [%s]"), *NativeName, *DisplayName)
			: NativeName;
	}

	return DisplayName;
}

}

void SCulturePicker::Construct( const FArguments& InArgs )
{
	OnCultureSelectionChanged = InArgs._OnSelectionChanged;
	IsCulturePickable = InArgs._IsCulturePickable;
	DisplayNameFormat = InArgs._DisplayNameFormat;
	ViewMode = InArgs._ViewMode;
	CanSelectNone = InArgs._CanSelectNone;

	BuildStockEntries();
	RebuildEntries();

	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSearchBox)
				.HintText( LOCTEXT("SearchHintText", "Name/Abbreviation") )
				.OnTextChanged(this, &SCulturePicker::OnFilterStringChanged)
				.DelayChangeNotificationsWhileTyping(true)
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FCultureEntry>>)
				.SelectionMode(ESelectionMode::Single)
				.TreeItemsSource(&RootEntries)
				.OnGenerateRow(this, &SCulturePicker::OnGenerateRow)
				.OnGetChildren(this, &SCulturePicker::OnGetChildren)
				.OnSelectionChanged(this, &SCulturePicker::OnSelectionChanged)
			]
		];


	if (TSharedPtr<FCultureEntry> InitialSelection = FindEntryForCulture(InArgs._InitialSelection))
	{
		TGuardValue<bool> SuppressSelectionGuard(SuppressSelectionCallback, true);
		TreeView->SetSelection(InitialSelection);
	}

	AutoExpandEntries();
}

void SCulturePicker::RequestTreeRefresh()
{
	RebuildEntries();
	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();
	}
	AutoExpandEntries();
}

TSharedPtr<FCultureEntry> SCulturePicker::FindEntryForCulture(FCulturePtr Culture) const
{
	return FindEntryForCultureImpl(Culture, RootEntries);
}

TSharedPtr<FCultureEntry> SCulturePicker::FindEntryForCultureImpl(FCulturePtr Culture, const TArray<TSharedPtr<FCultureEntry>>& Entries) const
{
	for (const TSharedPtr<FCultureEntry>& Entry : Entries)
	{
		if (Entry->Culture == Culture)
		{
			return Entry;
		}
		if (TSharedPtr<FCultureEntry> ChildEntry = FindEntryForCultureImpl(Culture, Entry->Children))
		{
			return ChildEntry;
		}
	}
	return nullptr;
}

void SCulturePicker::AutoExpandEntries()
{
	return AutoExpandEntriesImpl(RootEntries);
}

void SCulturePicker::AutoExpandEntriesImpl(const TArray<TSharedPtr<FCultureEntry>>& Entries)
{
	if (ViewMode == ECulturesViewMode::Hierarchical && TreeView)
	{
		for (const TSharedPtr<FCultureEntry>& Entry : Entries)
		{
			if (Entry->AutoExpand)
			{
				TreeView->SetItemExpansion(Entry, true);
				
				// Only recurse when actually filtering
				if (!FilterString.IsEmpty())
				{
					AutoExpandEntriesImpl(Entry->Children);
				}
			}
		}
	}
}

void SCulturePicker::BuildStockEntries()
{
	StockEntries.Empty();

	TArray<FString> StockCultureNames;
	FInternationalization::Get().GetCultureNames(StockCultureNames);

	TMap<FString, TSharedPtr<FCultureEntry>> TopLevelStockCultureEntries;
	TMap<FString, TSharedPtr<FCultureEntry>> AllStockCultureEntries;
	AllStockCultureEntries.Reserve(StockCultureNames.Num());

	for (const FString& CultureName : StockCultureNames)
	{
		const FCulturePtr Culture = FInternationalization::Get().GetCulture(CultureName);
		if (Culture.IsValid())
		{
			TArray<FString> HierarchicalCultureNames = Culture->GetPrioritizedParentCultureNames();
			if (HierarchicalCultureNames.Num() == 0 || HierarchicalCultureNames[0] != CultureName)
			{
				HierarchicalCultureNames.Remove(CultureName);
				HierarchicalCultureNames.Insert(CultureName, 0);
			}

			// Walk the array backwards to process the cultures in parent->child order
			TSharedPtr<FCultureEntry> ParentCultureEntry;
			const int32 TopLevelCultureIndex = HierarchicalCultureNames.Num() - 1;
			for (int32 CultureIndex = TopLevelCultureIndex; CultureIndex >= 0; --CultureIndex)
			{
				// Find the culture data
				const FString HierarchicalCultureName = HierarchicalCultureNames[CultureIndex];
				const FCulturePtr HierarchicalCulture = FInternationalization::Get().GetCulture(HierarchicalCultureName);
				if (!HierarchicalCulture.IsValid())
				{
					continue;
				}

				// Find or add a map entry for this culture
				TSharedPtr<FCultureEntry>& StockCultureEntryRef = AllStockCultureEntries.FindOrAdd(HierarchicalCultureName);
				if (!StockCultureEntryRef.IsValid())
				{
					StockCultureEntryRef = MakeShareable(new FCultureEntry(HierarchicalCulture));

					// Link this entry as a child of its parent
					if (ParentCultureEntry.IsValid())
					{
						ParentCultureEntry->Children.Add(StockCultureEntryRef);
					}
				}

				// Is this culture a top-level entry?
				if (CultureIndex == TopLevelCultureIndex)
				{
					TSharedPtr<FCultureEntry>& TopLevelStockCultureEntryRef = TopLevelStockCultureEntries.FindOrAdd(HierarchicalCultureName);
					if (!TopLevelStockCultureEntryRef.IsValid())
					{
						TopLevelStockCultureEntryRef = StockCultureEntryRef;
					}
				}

				ParentCultureEntry = StockCultureEntryRef;
			}
		}
	}

	// Populate the top-level array
	StockEntries.Reserve(TopLevelStockCultureEntries.Num());
	for (const auto& CultureNameDataPair : TopLevelStockCultureEntries)
	{
		StockEntries.Add(CultureNameDataPair.Value);
	}

	// Sort entries
	StockEntries.Sort([this](const TSharedPtr<FCultureEntry>& LHS, const TSharedPtr<FCultureEntry>& RHS) -> bool
	{
		const FString LHSDisplayName = GetCultureDisplayName(LHS->Culture.ToSharedRef(), false);
		const FString RHSDisplayName = GetCultureDisplayName(RHS->Culture.ToSharedRef(), false);
		return FTextComparison::CompareTo(LHSDisplayName, RHSDisplayName) < 0;
	});
}

void SCulturePicker::RebuildEntries()
{
	RootEntries.Reset();

	if (CanSelectNone)
	{
		RootEntries.Add(MakeShareable(new FCultureEntry(nullptr, true)));
	}
	
	TFunction<void (const TArray< TSharedPtr<FCultureEntry> >&, TArray< TSharedPtr<FCultureEntry> >&)> DeepCopyAndFilter
		= [&](const TArray< TSharedPtr<FCultureEntry> >& InEntries, TArray< TSharedPtr<FCultureEntry> >& OutEntries)
	{
		for (const TSharedPtr<FCultureEntry>& InEntry : InEntries)
		{
			// Set pickable flag.
			bool IsPickable = true;
			if (IsCulturePickable.IsBound())
			{
				IsPickable = IsCulturePickable.Execute(InEntry->Culture);
			}

			TSharedRef<FCultureEntry> OutEntry = MakeShareable(new FCultureEntry(InEntry->Culture, IsPickable));

			// Recurse to children.
			DeepCopyAndFilter(InEntry->Children, OutEntry->Children);

			bool IsFilteredOut = false;
			if (!FilterString.IsEmpty())
			{
				const FString Name = OutEntry->Culture->GetName();
				const FString DisplayName = OutEntry->Culture->GetDisplayName();
				const FString NativeName = OutEntry->Culture->GetNativeName();
				IsFilteredOut = !Name.Contains(FilterString) && !DisplayName.Contains(FilterString) && !NativeName.Contains(FilterString);
			}

			switch (ViewMode)
			{
			case ECulturesViewMode::Hierarchical:
				// If has children, must be added. If it is not filtered and it is pickable, should be added.
				if (OutEntry->Children.Num() != 0 || (!IsFilteredOut && IsPickable))
				{
					OutEntry->AutoExpand = (!IsPickable || IsFilteredOut) && OutEntry->Children.Num() > 0;
					OutEntries.Add(OutEntry);
				}
				break;

			case ECulturesViewMode::Flat:
				// Add this entry only if it's valid, but always append its children (as they have already been validated)
				if (IsPickable && !IsFilteredOut)
				{
					OutEntries.Add(OutEntry);
				}
				OutEntries.Append(OutEntry->Children);
				OutEntry->Children.Reset();
				break;

			default:
				checkf(false, TEXT("Unknown ECulturesViewMode!"));
				break;
			}
		}	
	};
	DeepCopyAndFilter(StockEntries, RootEntries);
}

void SCulturePicker::OnFilterStringChanged(const FText& InFilterString)
{
	FilterString = InFilterString.ToString();
	RequestTreeRefresh();
}

TSharedRef<ITableRow> SCulturePicker::OnGenerateRow(TSharedPtr<FCultureEntry> Entry, const TSharedRef<STableViewBase>& Table)
{
	return	SNew(STableRow< TSharedPtr<FCultureEntry> >, Table)
			[
				SNew(STextBlock)
				.Text(Entry->Culture.IsValid() ? FText::FromString(GetCultureDisplayName(Entry->Culture.ToSharedRef(), RootEntries.Contains(Entry))) : LOCTEXT("None", "None"))
				.ToolTip(
						SNew(SToolTip)
						.Content()
						[
							SNew(STextBlock)
							.Text(Entry->Culture.IsValid() ? FText::FromString(Entry->Culture->GetName()) : LOCTEXT("None", "None"))
							.HighlightText( FText::FromString(FilterString) )
						]
						)
				.HighlightText( FText::FromString(FilterString) )
				.ColorAndOpacity( Entry->IsSelectable ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground() )
			];
}

void SCulturePicker::OnGetChildren(TSharedPtr<FCultureEntry> Entry, TArray< TSharedPtr<FCultureEntry> >& Children)
{
	// Add entries from children array.
	Children.Append(Entry->Children);

	// Sort entries.
	Children.Sort([this](const TSharedPtr<FCultureEntry>& LHS, const TSharedPtr<FCultureEntry>& RHS) -> bool
	{
		const FString LHSDisplayName = GetCultureDisplayName(LHS->Culture.ToSharedRef(), false);
		const FString RHSDisplayName = GetCultureDisplayName(RHS->Culture.ToSharedRef(), false);
		return FTextComparison::CompareTo(LHSDisplayName, RHSDisplayName) < 0;
	});
}

void SCulturePicker::OnSelectionChanged(TSharedPtr<FCultureEntry> Entry, ESelectInfo::Type SelectInfo)
{
	if (SuppressSelectionCallback)
	{
		return;
	}

	// Don't count as selection if the entry isn't actually selectable but is part of the hierarchy.
	if (Entry.IsValid() && Entry->IsSelectable)
	{
		OnCultureSelectionChanged.ExecuteIfBound( Entry->Culture, SelectInfo );
	}
}

FString SCulturePicker::GetCultureDisplayName(const FCultureRef& Culture, const bool bIsRootItem) const
{
	return UE::Internationalization::Private::GetCultureDisplayName(Culture, DisplayNameFormat, bIsRootItem);
}

void SCulturePickerCombo::Construct(const FArguments& InArgs)
{
	SelectedCulture = InArgs._SelectedCulture;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	DisplayNameFormat = InArgs._DisplayNameFormat;

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(InArgs._Font)
			.Text(this, &SCulturePickerCombo::GetSelectedCultureDisplayName)
		]
		.MenuContent()
		[
			SNew(SBox)
			.MaxDesiredHeight(400.0f)
			.MaxDesiredWidth(300.0f)
			[
				SNew(SCulturePicker)
				.InitialSelection(SelectedCulture.Get(nullptr))
				.OnSelectionChanged(this, &SCulturePickerCombo::OnSelectedCultureChanged)
				.IsCulturePickable(InArgs._IsCulturePickable)
				.DisplayNameFormat(DisplayNameFormat)
				.ViewMode(InArgs._ViewMode)
				.CanSelectNone(InArgs._CanSelectNone)
			]
		]
	];
}

FText SCulturePickerCombo::GetSelectedCultureDisplayName() const
{
	FCulturePtr SelectedCulturePtr = SelectedCulture.Get(nullptr);
	return SelectedCulturePtr
		? FText::AsCultureInvariant(UE::Internationalization::Private::GetCultureDisplayName(SelectedCulturePtr.ToSharedRef(), DisplayNameFormat, false))
		: LOCTEXT("None", "None");
}

void SCulturePickerCombo::OnSelectedCultureChanged(FCulturePtr NewSelectedCulture, ESelectInfo::Type SelectInfo)
{
	if (!SelectedCulture.IsBound())
	{
		SelectedCulture = NewSelectedCulture;
	}
	OnSelectionChanged.ExecuteIfBound(NewSelectedCulture, SelectInfo);

	if (ComboButton)
	{
		ComboButton->SetIsOpen(false);
	}
}

#undef LOCTEXT_NAMESPACE
