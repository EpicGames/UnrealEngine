// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserGeneratedContentLocalizationDescriptorDetails.h"
#include "UserGeneratedContentLocalization.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#include "SCulturePicker.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SMultipleOptionTable.h"

#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "UserGeneratedContentLocalizationDescriptorDetails"

void FUserGeneratedContentLocalizationDescriptorDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (StructPropertyHandle->HasMetaData("ShowOnlyInnerProperties"))
	{
		return;
	}

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FUserGeneratedContentLocalizationDescriptorDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Build the list of available cultures used by the child customizations
	{
		FUserGeneratedContentLocalizationDescriptor DefaultUGCLocDescriptor;
		DefaultUGCLocDescriptor.InitializeFromProject();

		const TArray<FCultureRef> LocalizedCultureList = FInternationalization::Get().GetAvailableCultures(DefaultUGCLocDescriptor.CulturesToGenerate, false);

		AvailableCultures.Reset(LocalizedCultureList.Num());
		Algo::Transform(LocalizedCultureList, AvailableCultures, [](const FCultureRef Culture) -> FCulturePtr { return Culture; });

		// Sort by the current display name so that the order in the "native language" picker and the "cultures to generate" list matches
		AvailableCultures.StableSort([](FCulturePtr One, FCulturePtr Two)
		{
			return FTextComparison::CompareTo(One->GetDisplayName(), Two->GetDisplayName()) < 0;
		});
	}

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		if (TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(ChildIndex))
		{
			const FName ChildPropertyName = ChildPropertyHandle->GetProperty()->GetFName();
			if (ChildPropertyName == GET_MEMBER_NAME_CHECKED(FUserGeneratedContentLocalizationDescriptor, NativeCulture))
			{
				CustomizeNativeCulture(ChildPropertyHandle.ToSharedRef(), StructBuilder, StructCustomizationUtils);
			}
			else if(ChildPropertyName == GET_MEMBER_NAME_CHECKED(FUserGeneratedContentLocalizationDescriptor, CulturesToGenerate))
			{
				CustomizeCulturesToGenerate(ChildPropertyHandle.ToSharedRef(), StructBuilder, StructCustomizationUtils);
			}
			else
			{
				StructBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
			}
		}
	}
}

void FUserGeneratedContentLocalizationDescriptorDetails::CustomizeNativeCulture(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	NativeCultureHandle = PropertyHandle;

	StructBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
		.RowTag(PropertyHandle->GetProperty()->GetFName())
		.PropertyHandleList({ PropertyHandle })
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SCulturePickerCombo)
			.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
			.SelectedCulture(this, &FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_GetCulture)
			.OnSelectionChanged(this, &FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_OnSelectionChanged)
			.IsCulturePickable(this, &FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_IsCulturePickable)
			.DisplayNameFormat(SCulturePicker::ECultureDisplayFormat::ActiveCultureDisplayName)
			.ViewMode(SCulturePicker::ECulturesViewMode::Flat)
		];
}

FCulturePtr FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_GetCulture() const
{
	FString NativeCultureName;
	NativeCultureHandle->GetValue(NativeCultureName);

	return NativeCultureName.IsEmpty()
		? nullptr
		: FInternationalization::Get().GetCulture(NativeCultureName);
}

bool FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_IsCulturePickable(FCulturePtr Culture) const
{
	return AvailableCultures.Contains(Culture);
}

void FUserGeneratedContentLocalizationDescriptorDetails::NativeCulture_OnSelectionChanged(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo)
{
	NativeCultureHandle->SetValue(SelectedCulture->GetName());
}

void FUserGeneratedContentLocalizationDescriptorDetails::CustomizeCulturesToGenerate(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	CulturesToGenerateHandle = PropertyHandle;

	StructBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
		.RowTag(PropertyHandle->GetProperty()->GetFName())
		.PropertyHandleList({ PropertyHandle })
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SMultipleOptionTable<FCulturePtr>, &AvailableCultures)
			.ListHeight(100.0f)
			.OnGenerateOptionWidget_Lambda([](FCulturePtr Culture) -> TSharedRef<SWidget>
			{
				return SNew(STextBlock)
					.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
					.Text(FText::AsCultureInvariant(Culture->GetDisplayName()))
					.ToolTipText(FText::AsCultureInvariant(Culture->GetName()));
			})
			.OnPreBatchSelect(this, &FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnPreBatchSelect)
			.OnPostBatchSelect(this, &FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnPostBatchSelect)
			.OnOptionSelectionChanged(this, &FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnCultureSelectionChanged)
			.IsOptionSelected(this, &FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_IsCultureSelected)
		];
}

void FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnPreBatchSelect()
{
	CulturesToGenerate_IsInBatchSelectOperation = true;
	CulturesToGenerateHandle->NotifyPreChange();
}

void FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnPostBatchSelect()
{
	CulturesToGenerateHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	CulturesToGenerateHandle->NotifyFinishedChangingProperties();
	CulturesToGenerate_IsInBatchSelectOperation = false;
}

void FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_OnCultureSelectionChanged(bool IsSelected, FCulturePtr Culture)
{
	if (!CulturesToGenerate_IsInBatchSelectOperation)
	{
		CulturesToGenerateHandle->NotifyPreChange();
	}

	CulturesToGenerateHandle->EnumerateRawData([IsSelected, &Culture](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
	{
		TArray<FString>* RawCulturesToGenerateArray = reinterpret_cast<TArray<FString>*>(RawData);
		if (IsSelected)
		{
			RawCulturesToGenerateArray->AddUnique(Culture->GetName());
		}
		else
		{
			RawCulturesToGenerateArray->Remove(Culture->GetName());
		}
		return true;
	});

	if (!CulturesToGenerate_IsInBatchSelectOperation)
	{
		CulturesToGenerateHandle->NotifyPostChange(IsSelected ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
		CulturesToGenerateHandle->NotifyFinishedChangingProperties();
	}
}

bool FUserGeneratedContentLocalizationDescriptorDetails::CulturesToGenerate_IsCultureSelected(FCulturePtr Culture) const
{
	FString CultureName = Culture->GetName();
	TSharedPtr<IPropertyHandleArray> CulturesToGenerateArray = CulturesToGenerateHandle->AsArray();
	checkf(CulturesToGenerateArray, TEXT("CulturesToGenerate was not an array!"));

	uint32 ElementCount = 0;
	CulturesToGenerateArray->GetNumElements(ElementCount);
	for (uint32 Index = 0; Index < ElementCount; ++Index)
	{
		const TSharedRef<IPropertyHandle> ElementPropertyHandle = CulturesToGenerateArray->GetElement(Index);

		FString CultureNameAtIndex;
		ElementPropertyHandle->GetValue(CultureNameAtIndex);

		if (CultureNameAtIndex == CultureName)
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
