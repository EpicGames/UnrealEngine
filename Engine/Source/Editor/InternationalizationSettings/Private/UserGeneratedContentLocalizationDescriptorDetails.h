// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/CulturePointer.h"

namespace ESelectInfo { enum Type : int; }

class FUserGeneratedContentLocalizationDescriptorDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FUserGeneratedContentLocalizationDescriptorDetails>();
	}

	//~ Begin IPropertyTypeCustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization Interface

private:
	void CustomizeNativeCulture(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils);
	FCulturePtr NativeCulture_GetCulture() const;
	bool NativeCulture_IsCulturePickable(FCulturePtr Culture) const;
	void NativeCulture_OnSelectionChanged(FCulturePtr SelectedCulture, ESelectInfo::Type SelectInfo);

	void CustomizeCulturesToGenerate(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils);
	void CulturesToGenerate_OnPreBatchSelect();
	void CulturesToGenerate_OnPostBatchSelect();
	void CulturesToGenerate_OnCultureSelectionChanged(bool IsSelected, FCulturePtr Culture);
	bool CulturesToGenerate_IsCultureSelected(FCulturePtr Culture) const;
	
	TArray<FCulturePtr> AvailableCultures;

	TSharedPtr<IPropertyHandle> NativeCultureHandle;

	TSharedPtr<IPropertyHandle> CulturesToGenerateHandle;

	bool CulturesToGenerate_IsInBatchSelectOperation = false;
};
