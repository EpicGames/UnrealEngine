// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "Styling/AppStyle.h"
#include "InternationalizationSettingsModel.h"
#include "InternationalizationSettingsModelDetails.h"
