// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

#define UE_API INTERNATIONALIZATIONSETTINGS_API

class IDetailLayoutBuilder;

class FInternationalizationSettingsModelDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
};

#undef UE_API
