// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IInternationalizationSettingsModule : public IModuleInterface
{
public:
	static inline IInternationalizationSettingsModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IInternationalizationSettingsModule >( "InternationalizationSettingsModule" );
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "InternationalizationSettingsModule" );
	}
};

