// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/CulturePointer.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define UE_API INTERNATIONALIZATIONSETTINGS_API

class FText;
class ITableRow;
class SComboButton;

struct FCultureEntry
{
	FCulturePtr Culture;
	TArray< TSharedPtr<FCultureEntry> > Children;
	bool IsSelectable;
	bool AutoExpand = false;

	FCultureEntry(const FCulturePtr& InCulture, const bool InIsSelectable = true)
		: Culture(InCulture)
		, IsSelectable(InIsSelectable)
	{}

	FCultureEntry(const FCultureEntry& Source)
		: Culture(Source.Culture)
		, IsSelectable(Source.IsSelectable)
	{
		Children.Reserve(Source.Children.Num());
		for (const auto& Child : Source.Children)
		{
			Children.Add( MakeShareable( new FCultureEntry(*Child) ) );
		}
	}
};

class SCulturePicker : public SCompoundWidget
{
public:
	/** A delegate type invoked when a selection changes somewhere. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsCulturePickable, FCulturePtr);
	typedef TSlateDelegates< FCulturePtr >::FOnSelectionChanged FOnSelectionChanged;

	/** Different display name formats that can be used for a culture */
	enum class ECultureDisplayFormat
	{
		/** Should the culture display the name used by the active culture? */
		ActiveCultureDisplayName,
		/** Should the culture display the name used by the given culture? (ie, localized in their own native culture) */
		NativeCultureDisplayName,
		/** Should the culture display both the active and native cultures name? (will appear as "<ActiveName> (<NativeName>)") */
		ActiveAndNativeCultureDisplayName,
		/** Should the culture display both the native and active cultures name? (will appear as "<NativeName> (<ActiveName>)") */
		NativeAndActiveCultureDisplayName,
	};

	enum class ECulturesViewMode
	{
		/** Display the cultures hierarchically in a tree */
		Hierarchical,
		/** Display the cultures as a flat list */
		Flat,
	};

public:
	SLATE_BEGIN_ARGS( SCulturePicker )
		: _DisplayNameFormat(ECultureDisplayFormat::ActiveCultureDisplayName)
		, _ViewMode(ECulturesViewMode::Hierarchical)
		, _CanSelectNone(false)
	{}
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT( FIsCulturePickable, IsCulturePickable )
		SLATE_ARGUMENT( FCulturePtr, InitialSelection )
		SLATE_ARGUMENT(ECultureDisplayFormat, DisplayNameFormat)
		SLATE_ARGUMENT(ECulturesViewMode, ViewMode)
		SLATE_ARGUMENT(bool, CanSelectNone)
	SLATE_END_ARGS()

	SCulturePicker()
	: DisplayNameFormat(ECultureDisplayFormat::ActiveCultureDisplayName)
	, CanSelectNone(false)
	, SuppressSelectionCallback(false)
	{}

	UE_API void Construct( const FArguments& InArgs );
	UE_API void RequestTreeRefresh();

private:
	UE_API TSharedPtr<FCultureEntry> FindEntryForCulture(FCulturePtr Culture) const;
	UE_API TSharedPtr<FCultureEntry> FindEntryForCultureImpl(FCulturePtr Culture, const TArray<TSharedPtr<FCultureEntry>>& Entries) const;

	UE_API void AutoExpandEntries();
	UE_API void AutoExpandEntriesImpl(const TArray<TSharedPtr<FCultureEntry>>& Entries);

	UE_API void BuildStockEntries();
	UE_API void RebuildEntries();

	UE_API void OnFilterStringChanged(const FText& InFilterString);
	UE_API TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FCultureEntry> Entry, const TSharedRef<STableViewBase>& Table);
	UE_API void OnGetChildren(TSharedPtr<FCultureEntry> Entry, TArray< TSharedPtr<FCultureEntry> >& Children);
	UE_API void OnSelectionChanged(TSharedPtr<FCultureEntry> Entry, ESelectInfo::Type SelectInfo);

private:
	UE_API FString GetCultureDisplayName(const FCultureRef& Culture, const bool bIsRootItem) const;

	TSharedPtr< STreeView< TSharedPtr<FCultureEntry> > > TreeView;

	/* The top level culture entries for all possible stock cultures. */
	TArray< TSharedPtr<FCultureEntry> > StockEntries;

	/* The top level culture entries to be displayed in the tree view. */
	TArray< TSharedPtr<FCultureEntry> > RootEntries;

	/* The string by which to filter cultures names. */
	FString FilterString;

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnCultureSelectionChanged;

	/** Delegate to invoke to set whether a culture is "pickable". */
	FIsCulturePickable IsCulturePickable;

	/** How should we display culture names? */
	ECultureDisplayFormat DisplayNameFormat;

	/** How should we display the list of cultures? */
	ECulturesViewMode ViewMode;

	/** Should a null culture option be available? */
	bool CanSelectNone;

	/* Flags that the selection callback shouldn't be called when selecting - useful for initial selection, for instance. */
	bool SuppressSelectionCallback;
};

class SCulturePickerCombo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCulturePickerCombo)
		: _DisplayNameFormat(SCulturePicker::ECultureDisplayFormat::ActiveCultureDisplayName)
		, _ViewMode(SCulturePicker::ECulturesViewMode::Hierarchical)
		, _CanSelectNone(false)
	{}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_EVENT(SCulturePicker::FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(SCulturePicker::FIsCulturePickable, IsCulturePickable)
		SLATE_ATTRIBUTE(FCulturePtr, SelectedCulture)
		SLATE_ARGUMENT(SCulturePicker::ECultureDisplayFormat, DisplayNameFormat)
		SLATE_ARGUMENT(SCulturePicker::ECulturesViewMode, ViewMode)
		SLATE_ARGUMENT(bool, CanSelectNone)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:
	UE_API FText GetSelectedCultureDisplayName() const;
	UE_API void OnSelectedCultureChanged(FCulturePtr NewSelectedCulture, ESelectInfo::Type SelectInfo);

	TAttribute<FCulturePtr> SelectedCulture;
	SCulturePicker::FOnSelectionChanged OnSelectionChanged;
	SCulturePicker::ECultureDisplayFormat DisplayNameFormat;

	TSharedPtr<SComboButton> ComboButton;
};

#undef UE_API
